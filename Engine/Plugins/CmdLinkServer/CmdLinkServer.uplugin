{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Command Link Server",
	"Description": "Listens to and runs console commands from CmdLink",
	"Category": "Other",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "http://epicgames.com",
	"DocsURL": "",
	"EnabledByDefault": true,
	"MarketplaceURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "CmdLinkServer",
			"Type": "Editor",
			"LoadingPhase": "PostDefault",
			"PlatformAllowList": [
				"Win64"
			]
		}
	]
}