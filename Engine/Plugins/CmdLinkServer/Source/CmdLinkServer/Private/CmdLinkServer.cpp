// Copyright Epic Games, Inc. All Rights Reserved.

#include "CmdLinkServer.h"

#include "Editor.h"
#include "Async/Async.h"
#include "Misc/CString.h"
#include "Misc/StringOutputDevice.h"
#include "Features/IModularFeatures.h"

#define LOCTEXT_NAMESPACE "FCmdLinkServerModule"


namespace UE::CmdLink
{
	bool bEnabled = false;
	void OnCmdLinkEnabledChanged(IConsoleVariable* Var);
	FAutoConsoleVariableRef CVarEnableCmdLink(
		TEXT("console.CmdLink.enable"),
		bEnabled,
		TEXT("Opens a pipe that runs commands passed as command line args to CmdLink.exe"),
		ECVF_Default);

	FString CLIPipeKey = TEXT("None");
	void OnCmdLinkKeyChanged(IConsoleVariable* Var);
	FAutoConsoleVariableRef CVarCmdLinkKey(
		TEXT("console.CmdLink.key"),
		CLIPipeKey,
		TEXT("Changes the name of the pipe used for a connection to CmdLink.exe"),
		FConsoleVariableDelegate::CreateStatic(OnCmdLinkKeyChanged),
		ECVF_Default);

	void OnCmdLinkEnabledChanged(IConsoleVariable* Var)
	{
		if (FCmdLinkServerModule* Server = FCmdLinkServerModule::Get())
		{
			if (bEnabled)
			{
				Server->Enable();
			}
			else
			{
				Server->Disable();
			}
		}
	}

	void OnCmdLinkKeyChanged(IConsoleVariable* Var)
	{
		if (FCmdLinkServerModule* Server = FCmdLinkServerModule::Get())
		{
			Server->OnKeyChanged(CLIPipeKey);
		}
	}
	
	extern UNREALED_API void(*GBeginAsyncCommand)(const FString&, const TArray<FString>&);
	extern UNREALED_API void(*GEndAsyncCommand)(const FString&, const TArray<FString>&);
}



// because we're using a platform pipe in async mode and UE doesn't have coroutines, we can use this state machine
// to change which methods are being ticked
struct FCmdLinkServerModule::FTickStateMachine
{
	FTickStateMachine(int32 inCurrentTask)
		: CurrentTask(inCurrentTask)
	{
		Ticker = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FTickStateMachine::Tick));
	}

	~FTickStateMachine()
	{
		FTSTicker::RemoveTicker(Ticker);
	}

	void Emplace(int32 Key, TFunction<int32(float)>&& Task)
	{
		Tasks.Emplace(Key, Task);
	}

	int32 GetCurrentTask() const
	{
		return CurrentTask;
	}

private:
	bool Tick(float dt)
	{
		CurrentTask = Tasks[CurrentTask](dt);
		return Tasks.Find(CurrentTask) != nullptr;
	}
	
	int32 CurrentTask = 0;
	TMap<int32, TFunction<int32(float)>> Tasks;
	FTSTicker::FDelegateHandle Ticker;
};

enum ETask
{
	BEGIN_CONNECT, // create pipe and open the connection
	SLEEP_FOR_RECONNECT, // if BEGIN_CONNECT fails, wait a bit before trying again
	AWAIT_CONNECT, // wait for client to be ready
	
	BEGIN_READ,	// begin read operation
	AWAIT_READ, // wait for read operation to complete
	
	RUN_COMMAND, // run the command
	AWAIT_COMMAND, // wait for command to complete
	
	REPLY, // send the next reply data in the queue
	AWAIT_REPLY, // wait for reply to finish send


	EXIT = -1 // stop ticking
};

void FCmdLinkServerModule::StartupModule()
{
	// Disable by default on build machines to prevent conflicts over pipe name
	if (GIsBuildMachine)
	{
		UE::CmdLink::CVarEnableCmdLink->Set(false, ECVF_SetByCode);
	}
	if (FParse::Param(FCommandLine::Get(), TEXT("cmdlink")))
	{
		UE::CmdLink::CVarEnableCmdLink->Set(true, ECVF_SetByCode);
	}

	if (UE::CmdLink::bEnabled)
	{
		Enable();
	}
	
	UE::CmdLink::CVarEnableCmdLink->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(UE::CmdLink::OnCmdLinkEnabledChanged));
	
	UE::CmdLink::GBeginAsyncCommand = [](const FString& CommandName, const TArray<FString>& Params)
	{
		if (FCmdLinkServerModule* Module = FCmdLinkServerModule::Get())
		{
			Module->BeginAsyncCommand(CommandName, Params);
		}
	};
	UE::CmdLink::GEndAsyncCommand = [](const FString& CommandName, const TArray<FString>& Params)
	{
		if (FCmdLinkServerModule* Module = FCmdLinkServerModule::Get())
		{
			Module->EndAsyncCommand(CommandName, Params);
		}
	};
}

void FCmdLinkServerModule::ShutdownModule()
{
	if (UE::CmdLink::bEnabled)
	{
		Disable();
	}
}

void FCmdLinkServerModule::Enable()
{
	bEnabled = true;
	if (!StateMachine.IsValid())
	{
		InitStateMachine();
	}
}

void FCmdLinkServerModule::Disable()
{
	bEnabled = false;
	if (StateMachine.IsValid() && (StateMachine->GetCurrentTask() == BEGIN_CONNECT || StateMachine->GetCurrentTask() == AWAIT_CONNECT))
	{
		OnPipeClosed();
		StateMachine.Reset();
	}
}

void FCmdLinkServerModule::OnKeyChanged(const FString& NewKey)
{
	if (StateMachine.IsValid() && (StateMachine->GetCurrentTask() == BEGIN_CONNECT || StateMachine->GetCurrentTask() == AWAIT_CONNECT))
	{
		OnPipeClosed();
		StateMachine.Reset();
		if (bEnabled)
		{
			InitStateMachine();
		}
	}
}

void FCmdLinkServerModule::BeginAsyncCommand(const FString& CommandName, const TArray<FString>& Params)
{
	if (StateMachine->GetCurrentTask() != ETask::RUN_COMMAND)
	{
		return;
	}
	
	if (CommandName != ArgV[1])
	{
		return;
	}

	// first two args are file path and commandName
	if (ArgV.Num() - 2 != Params.Num())
	{
		return;
	}

	for (int ArgI = 0; ArgI < Params.Num(); ++ArgI)
	{
		if (ArgV[ArgI + 2].Data != Params[ArgI])
		{
			return;
		}
	}

	bAwaitAsyncCommand = true;
}

void FCmdLinkServerModule::EndAsyncCommand(const FString& CommandName, const TArray<FString>& Params)
{
	if (!bAwaitAsyncCommand)
	{
		return;
	}
	
	if (StateMachine->GetCurrentTask() != ETask::RUN_COMMAND && StateMachine->GetCurrentTask() != ETask::AWAIT_COMMAND)
	{
		return;
	}
	
	// first two args are file path and commandName
	if (CommandName != ArgV[1])
	{
		return;
	}

	if (ArgV.Num() - 2 != Params.Num())
	{
		return;
	}

	for (int ArgI = 0; ArgI < Params.Num(); ++ArgI)
	{
		if (ArgV[ArgI + 2].Data != Params[ArgI])
		{
			return;
		}
	}

	bAwaitAsyncCommand = false;
}

bool FCmdLinkServerModule::Read()
{
	if (ArgC == ArgV.Num())
	{
		return NamedPipe.ReadInt32(ArgC);
	}
	
	if (ArgV.Num() < ArgC)
	{
		// make sure ArgV is reserved to the correct size so we don't invalidate pointers
		// as we add to the array
		ArgV.Reserve(ArgC);
		
		if (NextStringSize == 0)
		{
			return NamedPipe.ReadInt32(NextStringSize);
		}
		
		ArgV.Emplace(NextStringSize);
		const bool Result = NamedPipe.ReadBytes(NextStringSize, ArgV.Last());
		NextStringSize = 0;
		
		return Result;
	}
	
	ensure(false);
	return false;
}

bool FCmdLinkServerModule::Execute(FString& Response)
{
	// merge command into a single line
	FString Command;
	for (int Arg = 1; Arg < ArgV.Num(); ++Arg)
	{
		Command += ArgV[Arg].Data;
		if (Arg < ArgV.Num() - 1)
		{
			Command += " ";
		}
	}
	if (Command.IsEmpty())
	{
		// if a command wasn't sent, assume they need help
		Command = TEXT("help");
	}

	// try running command on every command executor until one is successful (this allows us to run both engine and python commands)
	TArray<IConsoleCommandExecutor*> CommandExecutors = IModularFeatures::Get().GetModularFeatureImplementations<IConsoleCommandExecutor>(TEXT("ConsoleCommandExecutor"));
	for (IConsoleCommandExecutor* CommandExecutor : CommandExecutors)
	{
		FStringOutputDevice Ar;
		Ar.SetAutoEmitLineTerminator(true);
		
		GLog->AddOutputDevice(&Ar);
		const bool bSuccess = CommandExecutor->Exec(*Command);
		GLog->RemoveOutputDevice(&Ar);
		
		if (bSuccess)
		{
			Response = Ar;
			return true;
		}
	}
	
	return false;
}

void FCmdLinkServerModule::InitStateMachine()
{
	// The CmdLinkServer expects messages from the client in the following format:
	// 1. int32: ArgC
	// repeat lines 2 and 3 ArgC times
	// 2. int32: ArgLen
	// 3. char[Arglen]: Arg (includes null terminating character)

	// the CmdLink client expects a response from the server in the following format:
	// 1. int32: responseLen
	// 2. char[responseLen] response (includes null terminating character)
	
	StateMachine = MakeShared<FTickStateMachine>(BEGIN_CONNECT);
	
	StateMachine->Emplace(BEGIN_CONNECT, [this](float dt)->int32
	{
		// if there's a key specified, include it in the name
		const FString PipeName = UE::CmdLink::CLIPipeKey == TEXT("None") ? TEXT("UnrealEngine-CLI") : TEXT("UnrealEngine-CLI-") + UE::CmdLink::CLIPipeKey;
		// Create the output pipe as a server...
		if (!NamedPipe.Create(FString::Printf(TEXT("\\\\.\\pipe\\%s"), *PipeName), true, true))
		{
			return SLEEP_FOR_RECONNECT; // wait 5 seconds and try again
		}
		if (!NamedPipe.OpenConnection())
		{
			UE_LOGF(LogEngine, Error, "Failed to open a connection on the CLI pipe");
			return SLEEP_FOR_RECONNECT; // wait 5 seconds and try again
		}
		return AWAIT_CONNECT;
	});
	
	StateMachine->Emplace(SLEEP_FOR_RECONNECT, [this](float dt)->int32
	{
		if (SleepConnectTimer <= 0.f)
		{
			SleepConnectTimer = 5.f; // wait for 5 seconds
		}
		SleepConnectTimer -= dt;
		if (SleepConnectTimer <= 0.f)
		{
			return (bEnabled) ? BEGIN_CONNECT : EXIT;
		}
		return SLEEP_FOR_RECONNECT;
	});
	
	StateMachine->Emplace(AWAIT_CONNECT, [this](float dt)->int32
	{
		if (!NamedPipe.UpdateAsyncStatus())
		{
			return OnPipeClosed();
		}
		
		if (NamedPipe.IsReadyForRW())
		{
			return BEGIN_READ;
		}
		return AWAIT_CONNECT;
	});
	
	StateMachine->Emplace(BEGIN_READ, [this](float dt)->int32
	{
		if (!NamedPipe.UpdateAsyncStatus())
		{
			return OnPipeClosed();
		}
		
		if (NamedPipe.IsReadyForRW())
		{
			if (!Read())
			{
				return OnPipeClosed();
			}
			return AWAIT_READ;
		}
		return BEGIN_READ;
	});
	
	StateMachine->Emplace(AWAIT_READ, [this](float dt)->int32
	{
		if (!NamedPipe.UpdateAsyncStatus())
		{
			return OnPipeClosed();
		}
		if (NamedPipe.IsReadyForRW())
		{
			if (ArgV.Num() < ArgC)
			{
				// we need to read more data
				return BEGIN_READ;
			}
			// success
			return RUN_COMMAND;
		}
		return AWAIT_READ;
	});
	
	StateMachine->Emplace(RUN_COMMAND, [this](float dt)->int32
	{
		FString Response;
		const bool bSuccess = Execute(Response);
		const TStringConversion Result = StringCast<ANSICHAR>(*Response);
		// queue up response to the client
		if (Result.Length() == 0 && bSuccess == false)
		{
			const ANSICHAR Error[] = "Command failed or unrecognized";
			const int32 Size = TCString<ANSICHAR>::Strlen(Error) + 1;
			PendingReply.Enqueue(TArray(reinterpret_cast<const uint8*>(&Size), sizeof(Size)));
			PendingReply.Enqueue(TArray(reinterpret_cast<const uint8*>(Error), Size));
		}
		else
		{
			const int32 Size = Result.Length() + 1;
			PendingReply.Enqueue(TArray(reinterpret_cast<const uint8*>(&Size), sizeof(Size)));
			PendingReply.Enqueue(TArray(reinterpret_cast<const uint8*>(Result.Get()), Size));
		}
		
		if (bAwaitAsyncCommand)
		{
			return AWAIT_COMMAND;
		}
		return REPLY;
	});
	
	StateMachine->Emplace(AWAIT_COMMAND, [this](float dt)->int32
	{
		if (bAwaitAsyncCommand)
		{
			return AWAIT_COMMAND;
		}
		return REPLY;
	});
	
	StateMachine->Emplace(REPLY, [this](float dt)->int32
	{
		
		if (!NamedPipe.UpdateAsyncStatus())
		{
			return OnPipeClosed();
		}
		
		if (NamedPipe.IsReadyForRW())
		{
			TArray<uint8> Buffer;
			PendingReply.Dequeue(Buffer);
			if (!NamedPipe.WriteBytes(Buffer.Num(), Buffer.GetData()))
			{
				return OnPipeClosed();
			}
			return AWAIT_REPLY;
		}
		return REPLY;
	});
	
	StateMachine->Emplace(AWAIT_REPLY, [this](float dt)->int32
	{
		if (!NamedPipe.UpdateAsyncStatus())
		{
			return OnPipeClosed();
		}
		if (NamedPipe.IsReadyForRW())
		{
			if (PendingReply.IsEmpty())
			{
				// success! loop back and wait for the next message
				ArgV.Reset();
				ArgC = 0;
				NextStringSize = 0;
				return BEGIN_READ;
			}
			return REPLY;
		}
		return AWAIT_REPLY;
	});
}

int32 FCmdLinkServerModule::OnPipeClosed()
{
	// pipe connection broke. reconnect
	NamedPipe.Destroy();
	ArgV.Reset();
	ArgC = 0;
	NextStringSize = 0;
	PendingReply.Empty();
	return (bEnabled) ? BEGIN_CONNECT : EXIT;
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FCmdLinkServerModule, CmdLinkServer)