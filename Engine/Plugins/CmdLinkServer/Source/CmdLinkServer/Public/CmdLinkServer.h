// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Windows/WindowsPlatformNamedPipe.h"
#include "Containers/Queue.h"

class FCmdLinkServerModule : public IModuleInterface
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	void Enable();
	void Disable();
	void OnKeyChanged(const FString& NewKey);
	static FCmdLinkServerModule* Get()
	{
		return FModuleManager::GetModulePtr<FCmdLinkServerModule>("CmdLinkServer");
	}

	// call this from a command that will take more than one frame to complete
	// this forces the CmdLink to wait until EndAsyncCommand is called before shutting down
	CMDLINKSERVER_API void BeginAsyncCommand(const FString& CommandName, const TArray<FString>& Params);
	
	// call this at the end of a command that took more than one frame to complete
	CMDLINKSERVER_API void EndAsyncCommand(const FString& CommandName, const TArray<FString>& Params);
private:
	bool Read();
	bool Execute(FString& Response);
	void InitStateMachine();
	int32 OnPipeClosed();
	
	// scoped ANSICHAR c-string buffer that auto-casts to an FString
	struct FAnsiStringBuffer
	{
		FAnsiStringBuffer(int32 Size) : Data(new ANSICHAR[Size]{0}) {}
		~FAnsiStringBuffer() {delete [] Data;}
		operator FString() const {return FString(Data);}
		operator ANSICHAR*() const {return Data;}
		
		ANSICHAR* Data;
	};
	
	struct FTickStateMachine;
	bool bEnabled = false;
	FPlatformNamedPipe NamedPipe;
	TSharedPtr<FTickStateMachine> StateMachine;
	int32 ArgC = 0;
	TArray<FAnsiStringBuffer> ArgV;
	int32 NextStringSize = 0;
	bool bAwaitAsyncCommand = false;
	TQueue<TArray<uint8>> PendingReply;
	float SleepConnectTimer = -1.f;
};
